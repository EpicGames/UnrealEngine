// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serializers/MovieSceneSectionSerialization.h"

#include "Math/MathFwd.h"
#include "Serialization/CustomVersion.h"

struct FActorFileHeader;
struct FActorProperty;
struct FAnimationFileHeader;
struct FColor;
struct FLinearColor;
struct FManifestFileHeader;
struct FManifestProperty;
struct FPropertyFileHeader;
struct FSerializedAnimation;
struct FSerializedTransform;
struct FSerializedTypeFileHeader;
struct FSpawnFileHeader;
struct FSpawnProperty;
struct FTransformFileHeader;
template <typename PropertyType> struct FSerializedProperty;

DEFINE_LOG_CATEGORY(MovieSceneSerialization);


const FGuid FTempCustomVersion::GUID(0xCB8AB0CD, 0xE78C4BDE, 0xA8621393, 0x14E9EF62);
FCustomVersionRegistration GRegisterLiveLinkMovieSectionCustomVersion(FTempCustomVersion::GUID, FTempCustomVersion::LatestVersion, TEXT("LiveLinkCustomVersion"));

const int64 MovieSceneSerializationNamespace::InvalidOffset = -1;
const float MovieSceneSerializationNamespace::SerializerSleepTime = 0.2f;
bool MovieSceneSerializationNamespace::bAutoSerialize = false;

/** ==  Template Statics Defined  --  **/
template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FPropertyFileHeader, FPropertyFileHeader>* TMovieSceneSerializer<FPropertyFileHeader, FPropertyFileHeader>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FPropertyFileHeader, FPropertyFileHeader>::Thread = nullptr;

template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FTransformFileHeader, FSerializedTransform>* TMovieSceneSerializer<FTransformFileHeader, FSerializedTransform>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FTransformFileHeader, FSerializedTransform>::Thread = nullptr;

template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FAnimationFileHeader, FSerializedAnimation>* TMovieSceneSerializer<FAnimationFileHeader, FSerializedAnimation>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FAnimationFileHeader, FSerializedAnimation>::Thread = nullptr;


template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FSpawnFileHeader, FSpawnProperty>* TMovieSceneSerializer<FSpawnFileHeader, FSpawnProperty>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FSpawnFileHeader, FSpawnProperty>::Thread = nullptr;

template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FPropertyFileHeader, FSerializedProperty<int64>>* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<int64>>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<int64>>::Thread = nullptr;

template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FPropertyFileHeader, FSerializedProperty<bool>>* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<bool>>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<bool>>::Thread = nullptr;

template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FPropertyFileHeader, FSerializedProperty<uint8>>* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<uint8>>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<uint8>>::Thread = nullptr;

template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FPropertyFileHeader, FSerializedProperty<float>>* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<float>>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<float>>::Thread = nullptr;

template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FPropertyFileHeader, FSerializedProperty<double>>* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<double>>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<double>>::Thread = nullptr;

template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FPropertyFileHeader, FSerializedProperty<FVector3f>>* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<FVector3f>>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<FVector3f>>::Thread = nullptr;

template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FPropertyFileHeader, FSerializedProperty<FVector3d>>* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<FVector3d>>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<FVector3d>>::Thread = nullptr;

template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FPropertyFileHeader, FSerializedProperty<FColor>>* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<FColor>>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<FColor>>::Thread = nullptr;

template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FPropertyFileHeader, FSerializedProperty<int32>>* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<int32>>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<int32>>::Thread = nullptr;

template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FPropertyFileHeader, FSerializedProperty<FString>>* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<FString>>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<FString>>::Thread = nullptr;

template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FPropertyFileHeader, FSerializedProperty<FLinearColor>>* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<FLinearColor>>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<FLinearColor>>::Thread = nullptr;

/** External Tempplate statics.  Need to define here due to linkage issues*/

/* From TakeRecorderActorSource */
template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FManifestFileHeader, FManifestProperty>* TMovieSceneSerializer<FManifestFileHeader, FManifestProperty>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FManifestFileHeader, FManifestProperty>::Thread = nullptr;

template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FActorFileHeader, FActorProperty>* TMovieSceneSerializer<FActorFileHeader, FActorProperty>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FActorFileHeader, FActorProperty>::Thread = nullptr;


/* From SerializedRecorder */
template<> SERIALIZEDRECORDERINTERFACE_API
FMovieSceneSerializerRunnable<FSerializedTypeFileHeader, FSerializedTypeFileHeader>* TMovieSceneSerializer<FSerializedTypeFileHeader, FSerializedTypeFileHeader>::Runnable = nullptr;
template<> SERIALIZEDRECORDERINTERFACE_API
FRunnableThread* TMovieSceneSerializer<FSerializedTypeFileHeader, FSerializedTypeFileHeader>::Thread = nullptr;

