// Copyright Epic Games, Inc. All Rights Reserved.

#include "HAL/Platform.h"
#include "ISerializedRecorder.h"
#include "ISerializedRecorderInterfaceModule.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"

FName ISerializedRecorder::ModularFeatureName(TEXT("ModularFeature_SerialzedRecorder"));

class FSerializedRecorderInterfaceModule : public ISerializedRecorderInterfaceModule
{
public:
	
	// IModuleInterface interface
	virtual void StartupModule() override
	{
		
	}

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}

private:

};

IMPLEMENT_MODULE(FSerializedRecorderInterfaceModule, SerializedRecorderInterface);
