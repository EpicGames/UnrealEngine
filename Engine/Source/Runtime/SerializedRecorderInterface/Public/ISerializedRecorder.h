// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

#define UE_API SERIALIZEDRECORDERINTERFACE_API


class UMovieSceneSequence;
class UWorld;

class  ISerializedRecorder : public IModularFeature
{
public:
	virtual ~ISerializedRecorder() = default;

	static UE_API FName ModularFeatureName;
	virtual bool LoadRecordedSequencerFile(UMovieSceneSequence* InMovieSceneSequence, UWorld *PlaybackContext, const FString& InFileName, TFunction<void()> InCompletionCallback) = 0;
};

#undef UE_API
