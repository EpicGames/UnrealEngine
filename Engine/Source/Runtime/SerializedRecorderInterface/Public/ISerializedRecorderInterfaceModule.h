// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


class ISerializedRecorderInterfaceModule
	: public IModuleInterface
{
public:

	static ISerializedRecorderInterfaceModule& Get()
	{
#if PLATFORM_IOS
        static ISerializedRecorderInterfaceModule& LiveLinkInterfaceModule = FModuleManager::LoadModuleChecked<ISerializedRecorderInterfaceModule>("LiveLinkInterface");
        return LiveLinkInterfaceModule;
#else
        return FModuleManager::LoadModuleChecked<ISerializedRecorderInterfaceModule>("Interface");
#endif
	}

public:

	/** Virtual destructor. */
	virtual ~ISerializedRecorderInterfaceModule() = default;
};

