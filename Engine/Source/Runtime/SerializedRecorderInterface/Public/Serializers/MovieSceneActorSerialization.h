// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "UObject/UnrealType.h"
#include "MovieSceneSectionSerialization.h"


struct FActorFileHeader
{
	static const int32 cVersion = 1;

	FActorFileHeader() : Version(cVersion)
	{
	}


	FActorFileHeader(const FString& InName, const FString& InLabel, const FName& InSerializedType, const FString& InClassName, bool SpawnedPost)
		: Version(cVersion)
		, SerializedType(InSerializedType)
		, UObjectName(InName)
		, Label(InLabel)
		, bRecordToPossessable(false)
		, bWasSpawnedPostRecord(SpawnedPost)
		, ClassName(InClassName)
		, TemplateName("None")
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FActorFileHeader& Header)
	{
		Ar << Header.Version;
		Ar << Header.SerializedType;
		Ar << Header.Guid;
		Ar << Header.UObjectName;
		Ar << Header.Label;
		Ar << Header.bRecordToPossessable;
		Ar << Header.bWasSpawnedPostRecord;
		Ar << Header.ClassName;
		Ar << Header.TemplateName;
		Ar << Header.FolderName;

		return Ar;
	}


	//DATA
	int32 Version;
	FName SerializedType;
	FGuid Guid;

	FString UObjectName;
	FString Label;
	bool bRecordToPossessable;
	bool bWasSpawnedPostRecord;
	FString ClassName;
	FString TemplateName;
	FName FolderName;
};

enum class EActoryPropertyType : uint8 {

	ComponentType,
	PropertyType,
	OtherType,
};

struct FActorProperty
{
	FActorProperty() = default;
	FActorProperty(const FString &InObjectName, const FName& InSerializedType, const FGuid& InGuid) :
		UObjectName(InObjectName)
		, SerializedType(InSerializedType)
		, Guid(InGuid)
		, Type(EActoryPropertyType::OtherType)
	{}
	friend FArchive& operator<<(FArchive& Ar, FActorProperty& Property)
	{

		Ar << Property.SerializedType;
		Ar << Property.Guid;
		Ar << Property.UObjectName;
		Ar << Property.Type;
		if (Property.Type == EActoryPropertyType::ComponentType)
		{
			Ar << Property.BindingName;
			Ar << Property.ClassName;
		}
		else if (Property.Type == EActoryPropertyType::PropertyType)
		{
			Ar << Property.PropertyName;
		}

		return Ar;
	}


	//DATA

	FString UObjectName;
	FName SerializedType;
	FGuid Guid;
	EActoryPropertyType  Type;
	FString PropertyName;
	FString BindingName;
	FString ClassName;

};


using FActorSerializedFrame = TMovieSceneSerializedFrame<FActorProperty>;
using FActorSerializer = TMovieSceneSerializer<FActorFileHeader, FActorProperty>;
