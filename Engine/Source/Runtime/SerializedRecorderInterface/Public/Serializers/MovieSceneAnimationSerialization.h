// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "UObject/UnrealType.h"
#include "MovieSceneSectionSerialization.h"


struct FAnimationFileHeader
{
	static const int32 cVersion = 1;

	FAnimationFileHeader() : Version(cVersion)
	{
	}


	FAnimationFileHeader(const FName& InSerializedType, FGuid& InGuid, float InIntervalTime)
		: Version(cVersion)
		, SerializedType(InSerializedType) 
		, Guid(InGuid)
		, IntervalTime(InIntervalTime)
	{};


	friend FArchive& operator<<(FArchive& Ar, FAnimationFileHeader& Header)
	{
		Ar << Header.Version;
		Ar << Header.SerializedType;
		Ar << Header.Guid;
		Ar << Header.ActorGuid;
		Ar << Header.IntervalTime;
		Ar << Header.StartTime;
		Ar << Header.AnimationTrackNames;
		return Ar;
	}

	void AddNewRawTrack(const FName& BoneTreeName)
	{
		AnimationTrackNames.Add(BoneTreeName);
	}

	//DATA
	int32 Version;
	FName SerializedType;
	FGuid Guid;
	FGuid ActorGuid;
	float IntervalTime;
	float StartTime;
	TArray<FName> AnimationTrackNames;
};

struct FSerializedAnimationPerFrame
{
	FSerializedAnimationPerFrame() = default;

	friend FArchive& operator<<(FArchive& Ar, FSerializedAnimationPerFrame& Animation)
	{
		Ar << Animation.BoneIndex;
		Ar << Animation.PosKey;
		Ar << Animation.RotKey;
		Ar << Animation.ScaleKey;
		return Ar;
	}

	int32 BoneIndex;
	FVector PosKey;
	FQuat RotKey;
	FVector ScaleKey;

};

struct FSerializedAnimation
{
	FSerializedAnimation() = default;

	void AddTransform(int32 BoneIndex, const FTransform& InTransform)
	{
		FSerializedAnimationPerFrame Frame;
		Frame.BoneIndex = BoneIndex;
		Frame.PosKey = InTransform.GetTranslation();
		Frame.RotKey = InTransform.GetRotation();
		Frame.ScaleKey = InTransform.GetScale3D();
		AnimationData.Emplace(Frame);
	}

	friend FArchive& operator<<(FArchive& Ar, FSerializedAnimation& Animation)
	{
		Ar << Animation.AnimationData;
		return Ar;
	}
	TArray<FSerializedAnimationPerFrame> AnimationData;

};

using FAnimationSerializedFrame = TMovieSceneSerializedFrame<FSerializedAnimation>;
using FAnimationSerializer = TMovieSceneSerializer<FAnimationFileHeader, FSerializedAnimation>;
