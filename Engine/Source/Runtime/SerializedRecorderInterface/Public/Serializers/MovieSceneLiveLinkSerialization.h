// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"

#include "LiveLinkRole.h"
#include "LiveLinkTypes.h"
#include "LiveLinkCustomVersion.h"

#include "Serializers/MovieSceneSectionSerialization.h"


/** !!! LiveLink Track serialization is not supported !!! */

struct FLiveLinkManifestHeader
{

	static const int32 cVersion = 2;

	FLiveLinkManifestHeader() : Version(cVersion)
	{
	}


	FLiveLinkManifestHeader(const FName& InSerializedType)
		: Version(cVersion)
		, SerializedType(InSerializedType)
		, bIsManifest(true)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FLiveLinkManifestHeader& Header)
	{
		Ar << Header.Version;
		Ar << Header.SerializedType;
		Ar << Header.bIsManifest;
		Ar << Header.SubjectNames;

		return Ar;
	}
	int32 Version;
	FName SerializedType;
	bool bIsManifest;
	TArray<FName> SubjectNames;
};

using FLiveLinkManifestSerializer = TMovieSceneSerializer<FLiveLinkManifestHeader, FLiveLinkManifestHeader>;


PRAGMA_DISABLE_DEPRECATION_WARNINGS
//Header and data for individual subjects that contain the data.
struct FLiveLinkFileHeader
{
	FLiveLinkFileHeader() = default;

	FLiveLinkFileHeader(const FName& InSubjectName, double InSecondsDiff, TSubclassOf<ULiveLinkRole> InSubjectRole, const TSharedPtr<FLiveLinkStaticDataStruct>& InStaticData, const FName& InSerializedType, const FGuid& InGuid)
		: bIsManifest(false)
		, SecondsDiff(InSecondsDiff)
		, SubjectName(InSubjectName)
		, SerializedType(InSerializedType)
		, Guid(InGuid)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FLiveLinkFileHeader& Header)
	{
		Ar.UsingCustomVersion(FLiveLinkCustomVersion::GUID);

		Ar << Header.SerializedType;
		Ar << Header.bIsManifest;
		Ar << Header.Guid;
		Ar << Header.SecondsDiff;
		Ar << Header.SubjectName;

		if (Ar.IsLoading())
		{
			if (Ar.CustomVer(FLiveLinkCustomVersion::GUID) < FLiveLinkCustomVersion::NewLiveLinkRoleSystem)
			{
				Ar << Header.CurveNames;
				FLiveLinkRefSkeleton::StaticStruct()->SerializeItem(Ar, (void*)& Header.RefSkeleton, nullptr);
			}
		}

		return Ar;
	}

	bool bIsManifest;
	double SecondsDiff;
	FName SubjectName;
	FName SerializedType;
	FGuid Guid;

	TArray<FName> CurveNames;
	FLiveLinkRefSkeleton RefSkeleton;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

