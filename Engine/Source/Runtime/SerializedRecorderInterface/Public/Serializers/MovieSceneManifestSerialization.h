// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "UObject/UnrealType.h"
#include "MovieSceneSectionSerialization.h"


struct FManifestFileHeader
{
	static const int32 cVersion = 1;

	FManifestFileHeader() : Version(cVersion)
	{
	}

	FManifestFileHeader(const FString& InName, const FName& InSerializedType, const FGuid& InGuid) 
		: Version(cVersion)
		, SerializedType(InSerializedType)
		, Guid(InGuid)
		, Name(InName)

	{
	}

	friend FArchive& operator<<(FArchive& Ar, FManifestFileHeader& Header)
	{
		Ar << Header.Version;
		Ar << Header.SerializedType;
		Ar << Header.Guid;
		Ar << Header.Name;

		return Ar;
	}

	int32 Version;
	FName SerializedType;
	FGuid Guid;
	FString Name;
};


//since we can't use function templates with TFunctions need to use a variant or union.
struct FManifestProperty
{
	FManifestProperty() = default;
	FManifestProperty(const FString &InObjectName, const FName& InSerializedType, const FGuid& InGuid) :
		UObjectName(InObjectName)
		, SerializedType(InSerializedType)
		, Guid(InGuid)
	{}
	friend FArchive& operator<<(FArchive& Ar, FManifestProperty& Property)
	{

		Ar << Property.SerializedType;
		Ar << Property.Guid;
		Ar << Property.UObjectName;
		return Ar;
	}


	//DATA

	FString UObjectName;
	FName SerializedType;
	FGuid Guid;

};


using FManifestSerializedFrame = TMovieSceneSerializedFrame<FManifestProperty>;
using FManifestSerializer = TMovieSceneSerializer<FManifestFileHeader, FManifestProperty>;