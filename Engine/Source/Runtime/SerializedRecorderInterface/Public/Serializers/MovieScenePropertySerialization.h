// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "UObject/StrProperty.h"
#include "UObject/UnrealType.h"
#include "UObject/EnumProperty.h"
#include "MovieSceneSectionSerialization.h"

enum class ESerializedPropertyType : uint8 {

	BoolType,
	ByteType,
	EnumType,
	FloatType,
	DoubleType,
	Vector3fType,
	Vector3dType,
	ColorType,
	IntegerType,
	StringType,
	LinearColorType
};

struct FPropertyFileHeader
{

	static const int32 cVersion = 1;

	FPropertyFileHeader() : Version(cVersion)
	{
	}


	FPropertyFileHeader(const FFrameRate &InFrameRate, const FName& InSerializedType, const FGuid& InGuid)
		: Version(cVersion)
		, SerializedType(InSerializedType)
		, Guid(InGuid)
		, TickResolution(InFrameRate)

	{
	}

	friend FArchive& operator<<(FArchive& Ar, FPropertyFileHeader& Header)
	{
		Ar << Header.Version;
		Ar << Header.SerializedType;
		Ar << Header.Guid;
		Ar << Header.TickResolution.Numerator;
		Ar << Header.TickResolution.Denominator;
		Ar << Header.PropertyName;
		Ar << Header.PropertyType;
		Ar << Header.TrackDisplayName;

		return Ar;
	}

	void SetProperty(const FProperty* Property, const FName& InPropertyName)
	{
		if (Property != nullptr)
		{
			PropertyName = InPropertyName;

		#if WITH_EDITOR
			TrackDisplayName = *Property->GetDisplayNameText().ToString();
		#endif // WITH_EDITOR

			if (Property->IsA<FBoolProperty>())
			{
				PropertyType = (ESerializedPropertyType::BoolType);
			}
			else if (Property->IsA<FByteProperty>())
			{
				PropertyType = (ESerializedPropertyType::ByteType);
			}
			else if (Property->IsA<FEnumProperty>())
			{
				PropertyType = (ESerializedPropertyType::EnumType);
			}
			else if (Property->IsA<FFloatProperty>())
			{
				PropertyType = (ESerializedPropertyType::FloatType);
			}
			else if (Property->IsA<FDoubleProperty>())
			{
				PropertyType = (ESerializedPropertyType::DoubleType);
			}
			else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(Property))
			{
				// LWC_TODO: vector types
				if (StructProperty->Struct->GetFName() == NAME_Vector3f)
				{
					PropertyType = (ESerializedPropertyType::Vector3fType);
				}
				else if (StructProperty->Struct->GetFName() == NAME_Vector3d
						|| StructProperty->Struct->GetFName() == NAME_Vector
						)
				{
					PropertyType = (ESerializedPropertyType::Vector3dType);
				}
				else if (StructProperty->Struct->GetFName() == NAME_Color)
				{
					PropertyType = (ESerializedPropertyType::ColorType);
				}
				else if (StructProperty->Struct->GetFName() == NAME_LinearColor)
				{
					PropertyType = (ESerializedPropertyType::LinearColorType);
				}
			}
			else if (Property->IsA<FIntProperty>())
			{
				PropertyType = (ESerializedPropertyType::IntegerType);
			}
			else if (Property->IsA<FStrProperty>())
			{
				PropertyType = (ESerializedPropertyType::StringType);
			}
		}
	}


	//DATA
	int32 Version;
	FName SerializedType;
	FGuid Guid;
	FFrameRate TickResolution;
	FName PropertyName;
	ESerializedPropertyType PropertyType;
	FString TrackDisplayName;
};


template <typename PropertyType>
struct FSerializedProperty
{
	FSerializedProperty() = default;

	friend FArchive& operator<<(FArchive& Ar, FSerializedProperty& Property)
	{
		Ar << Property.Time;
		Ar << Property.Value;
		return Ar;
	}

	FFrameNumber Time;
	PropertyType Value;

};

using FPropertySerializedBool = FSerializedProperty<bool>;
using FPropertySerializedBoolFrame = TMovieSceneSerializedFrame<FSerializedProperty<bool>>;

using FPropertySerializedByte = FSerializedProperty<uint8>;
using FPropertySerializedByteFrame = TMovieSceneSerializedFrame<FSerializedProperty<uint8>>;

using FPropertySerializedEnum = FSerializedProperty<int64>;
using FPropertySerializedEnumFrame = TMovieSceneSerializedFrame<FSerializedProperty<int64>>;
using FPropertySerializerEnum = TMovieSceneSerializer<FPropertyFileHeader, FSerializedProperty<int64>>;

using FPropertySerializedDouble = FSerializedProperty<double>;
using FPropertySerializedDoubleFrame = TMovieSceneSerializedFrame<FSerializedProperty<double>>;

using FPropertySerializedFloat = FSerializedProperty<float>;
using FPropertySerializedFloatFrame = TMovieSceneSerializedFrame<FSerializedProperty<float>>;

using FPropertySerializedVector3f = FSerializedProperty<FVector3f>;
using FPropertySerializedVector3fFrame = TMovieSceneSerializedFrame<FSerializedProperty<FVector3f>>;

using FPropertySerializedVector3d = FSerializedProperty<FVector3d>;
using FPropertySerializedVector3dFrame = TMovieSceneSerializedFrame<FSerializedProperty<FVector3d>>;

using FPropertySerializedColor = FSerializedProperty<FColor>;
using FPropertySerializedColorFrame = TMovieSceneSerializedFrame<FSerializedProperty<FColor>>;

using FPropertySerializedInteger = FSerializedProperty<int32>;
using FPropertySerializedIntegerFrame = TMovieSceneSerializedFrame<FSerializedProperty<int32>>;

using FPropertySerializedString = FSerializedProperty<FString>;
using FPropertySerializedStringFrame = TMovieSceneSerializedFrame<FSerializedProperty<FString>>;

using FPropertySerializedLinearColor = FSerializedProperty<FLinearColor>;
using FPropertySerializedLinearColorFrame = TMovieSceneSerializedFrame<FSerializedProperty<FLinearColor>>;