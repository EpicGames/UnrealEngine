// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/BinarySearch.h"
#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Containers/LockFreeList.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/CriticalSection.h"
#include "HAL/FileManager.h"
#include "HAL/FileManagerGeneric.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformProperties.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/ThreadSafeBool.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/App.h"
#include "Misc/AssertionMacros.h"
#include "Misc/DateTime.h"
#include "Misc/FrameRate.h"
#include "Misc/Guid.h"
#include "Misc/NetworkVersion.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Serialization/Archive.h"
#include "Serialization/ArchiveProxy.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/NameAsStringProxyArchive.h"
#include "Templates/Function.h"
#include "Templates/Less.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"

SERIALIZEDRECORDERINTERFACE_API DECLARE_LOG_CATEGORY_EXTERN(MovieSceneSerialization, Verbose, All);

namespace MovieSceneSerializationNamespace
{
	extern SERIALIZEDRECORDERINTERFACE_API const int64 InvalidOffset;
	extern SERIALIZEDRECORDERINTERFACE_API const float SerializerSleepTime;
	extern SERIALIZEDRECORDERINTERFACE_API bool bAutoSerialize;
};

template <typename FrameData>
struct TMovieSceneSerializedFrame
{
	uint64 FrameId;
	FrameData  Frame;

	friend FArchive& operator<<(FArchive& Ar, TMovieSceneSerializedFrame& InFrame)
	{
		Ar << InFrame.FrameId;
		Ar << InFrame.Frame;
		return Ar;
	}
};


/** Header (non-tagged property serialized) for a frame */
struct FMovieSceneSerializedFrameHeader
{
	FMovieSceneSerializedFrameHeader()
		: NextFrameOffset(MovieSceneSerializationNamespace::InvalidOffset)
		, PrevFrameOffset(MovieSceneSerializationNamespace::InvalidOffset)
		, FrameId(0)
	{}

	FMovieSceneSerializedFrameHeader(int64 InNextFrameOffset, int64 InPrevFrameOffset, uint64 InFrameId)
		: NextFrameOffset(InNextFrameOffset)
		, PrevFrameOffset(InPrevFrameOffset)
		, FrameId(InFrameId)
	{}

	/** Offset within the archive that the next frame can be found. -1 if this is the last frame. */
	int64 NextFrameOffset;

	/** Offset within the archive that the next frame can be found. -1 if this is the first frame. */
	int64 PrevFrameOffset;

	/** ID of this frame */
	uint64 FrameId;


	friend FArchive& operator<<(FArchive& Ar, FMovieSceneSerializedFrameHeader& Header)
	{
		Ar << Header.NextFrameOffset;
		Ar << Header.PrevFrameOffset;
		Ar << Header.FrameId;

		return Ar;
	}
};

class FMovieSceneArchiveFileReader : public FArchiveFileReaderGeneric
{
public:
	FMovieSceneArchiveFileReader(IFileHandle* InHandle, const TCHAR* InFilename)
		: FArchiveFileReaderGeneric(InHandle, InFilename, InHandle->Size())
	{}

	void BeginArchiving()
	{
		Size = Handle->Size();
		Pos = Handle->Tell();
		BufferBase = 0;
		BufferArray.Reset();
	}

	/** FArchiveFileReaderGeneric interface*/
	virtual void CloseLowLevel() override
	{
		// Release the handle as this archive doesnt actually own it
		(void)Handle.Release();
	}
};

class FMovieSceneArchiveFileWriter : public FArchiveFileWriterGeneric
{
public:
	FMovieSceneArchiveFileWriter(IFileHandle* InHandle, const TCHAR* InFilename)
		: FArchiveFileWriterGeneric(InHandle, InFilename, InHandle->Tell())
	{}

	void BeginArchiving()
	{
		Pos = Handle->Tell();
		BufferArray.Reset();
	}

	/** FArchiveFileWriterGeneric interface*/
	virtual bool CloseLowLevel() override
	{
		// Release the handle as this archive doesnt actually own it
		(void)Handle.Release();
		return true;
	}
};




/** Information about a captured session */
struct FMovieSceneSessionInfo
{
	FMovieSceneSessionInfo()
		: BuildVersion(0)
	{}

	FMovieSceneSessionInfo(const FGuid& InInstanceId, FDateTime InTimeStamp, int32 InBuildVersion, const FString& InDeviceName, const FString& InPlatformName, const FString& InInstanceName, const FString& InBuildDate)
		: InstanceId(InInstanceId)
		, TimeStamp(InTimeStamp)
		, BuildVersion(InBuildVersion)
		, DeviceName(InDeviceName)
		, PlatformName(InPlatformName)
		, InstanceName(InInstanceName)
		, BuildDate(InBuildDate)
	{}

	/** The ID of the session instance the file was captured with */
	FGuid InstanceId;

	/** The timestamp of this capture at creation time. @see FDateTime::UtcNow() */
	FDateTime TimeStamp;

	/** The engine version the file was captured with. @see ISessionInstanceInfo */
	int32 BuildVersion;

	/** Name of the device the file was captured on. @see ISessionInstanceInfo */
	FString DeviceName;

	/** Name of the platform the file was captured on. @see ISessionInstanceInfo */
	FString PlatformName;

	/** Name of the engine instance the file was captured on. @see ISessionInstanceInfo */
	FString InstanceName;

	/** The build date for the capture. @see ISessionInstanceInfo */
	FString BuildDate;

	friend FArchive& operator<<(FArchive& Ar, FMovieSceneSessionInfo& InOutSessionInfo)
	{
		Ar << InOutSessionInfo.InstanceId;
		Ar << InOutSessionInfo.TimeStamp;
		Ar << InOutSessionInfo.BuildVersion;
		Ar << InOutSessionInfo.DeviceName;
		Ar << InOutSessionInfo.PlatformName;
		Ar << InOutSessionInfo.InstanceName;
		Ar << InOutSessionInfo.BuildDate;

		return Ar;
	}
};




/** Archive context maintained when the serializer is running */
class FMovieSceneSerializerContext
{
public:

	FMovieSceneSerializerContext(IFileHandle* InHandle, const FString& InFilename)
		: Filename(InFilename)
		, Handle(InHandle)
		, ArReaderInner(InHandle, *Filename)
		, ArWriterInner(InHandle, *Filename)
		, ArReader(ArReaderInner)
		, ArWriter(ArWriterInner)
		, LastFrameWritePos(MovieSceneSerializationNamespace::InvalidOffset)
		, MinFrameId(0xffffffffffffffff)
		, MaxFrameId(0)
	{}

	~FMovieSceneSerializerContext() = default;

	void Close()
	{
		ArReader.Close();
		ArWriter.Close();
		delete Handle;
		Handle = nullptr;
	}


public:
	
	/** Filename */
	FString Filename;

	/** Read/write handles */
	IFileHandle* Handle;

	/** Inner file reader/writer archives */
	FMovieSceneArchiveFileReader ArReaderInner;
	FMovieSceneArchiveFileWriter ArWriterInner;

	/** Reader/writer archives */
	FNameAsStringProxyArchive ArReader;
	FNameAsStringProxyArchive ArWriter;

	/** Offset within the archive that the last frame was written */
	int64 LastFrameWritePos;

	/** Frame IDs used to access specific times quickly */
	TArray<TPair<uint64, int64>> FrameIdToFrameOffset;

	/** The min frame ID contained in the file */
	uint64 MinFrameId;

	/** The max frame ID contained in the file */
	uint64 MaxFrameId;

};


typedef TFunction<void(FMovieSceneSerializerContext& /*InContext*/)> FSerializerCommand;

struct FTempCustomVersion
{
	enum Type
	{
		// Initial version
		FirstVersion = 0,

		// -----new versions can be added above this line-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	SERIALIZEDRECORDERINTERFACE_API const static FGuid GUID;
private:
	FTempCustomVersion() {}
};


template <typename HeaderData, typename FrameData>
struct FContextAndCommands
{
	FContextAndCommands(IFileHandle* InHandle, const FString& InFilename)
		: Context(InHandle, InFilename)
	{

	}
	~FContextAndCommands()
	{
		while (FSerializerCommand* Command = ReadCommands.Pop())
		{
			delete Command;
		}

		while (FSerializerCommand* Command = WriteCommands.Pop())
		{
			delete Command;
		}

		while (FSerializerCommand* Command = FreeCommands.Pop())
		{
			delete Command;
		}
	}
	/** Unique ID of this Guid, key for a map of Context*/
	FGuid Guid;

	FMovieSceneSerializerContext Context;

	/** Read command queue - pushed on any thread, popped on the serializer worker thread */
	TLockFreePointerListFIFO<FSerializerCommand, 0> ReadCommands;

	/** Write command queue - pushed on any thread, popped on the serializer worker thread */
	TLockFreePointerListFIFO<FSerializerCommand, 0> WriteCommands;

	/** Free command queue - pushed on any thread, popped on any thread */
	TLockFreePointerListFIFO<FSerializerCommand, 0> FreeCommands;
	
	void Close()
	{
		Context.Close();
	}

	void Run()
	{
		// Pop and run our commands
		TArray<FSerializerCommand*> LocalReadCommands;
		ReadCommands.PopAll(LocalReadCommands);

		TArray<FSerializerCommand*> LocalWriteCommands;
		WriteCommands.PopAll(LocalWriteCommands);

		if (LocalReadCommands.Num() > 0)
		{
			Context.ArReaderInner.BeginArchiving();

			for (FSerializerCommand* Command : LocalReadCommands)
			{
				if (Command != nullptr)
				{
					(*Command)(Context);
					FreeCommands.Push(Command);
				}
			}

			Context.ArReader.Flush();
		}

		if (LocalWriteCommands.Num() > 0)
		{
			Context.ArWriterInner.BeginArchiving();

			for (FSerializerCommand* Command : LocalWriteCommands)
			{
				if (Command != nullptr)
				{
					(*Command)(Context);
					FreeCommands.Push(Command);
				}
			}

			Context.ArWriter.Flush();
		}
	}


	bool CheckHeader(bool bInFileExisted, TArray<FMovieSceneSessionInfo>& SessionInfos, HeaderData& InOutHeader, int64& ReadHeaderPos,  FText& OutFailReason)
	{

		Context.ArReader.UsingCustomVersion(FTempCustomVersion::GUID);
		Context.ArWriter.UsingCustomVersion(FTempCustomVersion::GUID);

		if (bInFileExisted)
		{
			// If we are reading an existing file, check the version
			int32 ArchiveVer = INT_MAX;
			Context.ArReader << ArchiveVer;

			if (ArchiveVer > FTempCustomVersion::LatestVersion)
			{
				OutFailReason = FText::Format(NSLOCTEXT("MovieSceneSerializer", "FileOpenFailedIncorrectVersion", "Cannot open file.\nFile version: {0}.\nMax supported version: {1}."), FText::AsNumber(ArchiveVer), FText::AsNumber((int32)FTempCustomVersion::LatestVersion));
				UE_LOGF(MovieSceneSerialization, Error, "Cannot open Movie Scene Serialization cache %ls. File version: %d. Max supported version: %d", *Context.Filename, ArchiveVer, FTempCustomVersion::LatestVersion);
				Context.Close();
				return false;
			}


			int32 NumSessions = SessionInfos.Num();
			Context.ArReader << NumSessions;

			if (Context.ArReader.IsLoading())
			{
				SessionInfos.SetNum(NumSessions);
			}

			for (int32 SessionIndex = 0; SessionIndex < NumSessions; ++SessionIndex)
			{
				Context.ArReader << SessionInfos[SessionIndex];
			}

			Context.ArReader << InOutHeader;

			Context.ArReader.Flush();
		}
		else
		{
			int32 ArchiveVer = Context.ArWriter.CustomVer(FTempCustomVersion::GUID);
			Context.ArWriter << ArchiveVer;


			int32 NumSessions = SessionInfos.Num();
			Context.ArWriter << NumSessions;
			for (int32 SessionIndex = 0; SessionIndex < NumSessions; ++SessionIndex)
			{
				Context.ArWriter << SessionInfos[SessionIndex];
			}

			Context.ArWriter << InOutHeader;
			Context.ArWriter.Flush();
		}
		ReadHeaderPos = Context.ArReader.Tell();

		return true;
	}

	
	/**
	* Gets a free command from the free list, if any. Otherwise allocates a new command. This means
	* that the number of commands grows unbounded to the high watermark of all commands.
	*/
	template <typename... ArgsType>
	FORCEINLINE FSerializerCommand* GetFreeCommand(ArgsType&&... Args)
	{
		FSerializerCommand* Command = FreeCommands.Pop();
		if (Command == nullptr)
		{
			Command = new FSerializerCommand(Forward<ArgsType>(Args)...);
		}
		else
		{
			*Command = FSerializerCommand(Forward<ArgsType>(Args)...);
		}

		return Command;
	}

	template <typename... ArgsType>
	FORCEINLINE void AddReadCommand(ArgsType&&... Args)
	{
		ReadCommands.Push(GetFreeCommand(Forward<ArgsType>(Args)...));
	}

	template <typename... ArgsType>
	FORCEINLINE void AddWriteCommand(ArgsType&&... Args)
	{
		WriteCommands.Push(GetFreeCommand(Forward<ArgsType>(Args)...));
	}
};

/** Runnable to run threaded I/O */
template <typename HeaderData, typename FrameData>
class FMovieSceneSerializerRunnable : public FRunnable
{
public:
	FMovieSceneSerializerRunnable()
		: bRunning(true), bStopping(false)
	{}

	~FMovieSceneSerializerRunnable()
	{
		check(!bRunning);

	
	}

	void AddContext(const FGuid& InGuid, IFileHandle* InHandle, const FString& InFileName)
	{
		FScopeLock Lock(&ContextMapCriticalSection);
		TSharedPtr<FContextAndCommands<HeaderData, FrameData>> Context = MakeShared<FContextAndCommands<HeaderData, FrameData>>(InHandle, InFileName);
		ContextMap.Add(InGuid, Context);
	}

	/** FRunnable interface */
	virtual uint32 Run() override
	{
		while (bRunning)
		{
			{
				FScopeLock Lock(&ContextMapCriticalSection);

				for (auto& Entry : ContextMap)
				{
					TSharedPtr <FContextAndCommands<HeaderData, FrameData>> & Context = Entry.Value;
					Context->Run();
				}
			}

			if (bStopping)
			{
				bRunning = false;
			}
			FPlatformProcess::Sleep(MovieSceneSerializationNamespace::SerializerSleepTime);
		}

		CloseContexts();
		return 0;
	}

	virtual void Stop() override
	{
		bStopping = true;

	}

	bool CheckHeader(const FGuid& InGuid, bool bInFileExisted, TArray<FMovieSceneSessionInfo>& SessionInfos, HeaderData& InOutHeader, int64& ReadHeaderPos,  FText& OutFailReason)
	{
		FScopeLock Lock(&ContextMapCriticalSection);
		TSharedPtr<FContextAndCommands<HeaderData, FrameData >>* Context = ContextMap.Find(InGuid);
		if (Context)
		{
			return (*Context)->CheckHeader(bInFileExisted, SessionInfos, InOutHeader, ReadHeaderPos, OutFailReason);
		}

		return false;
	}

	FContextAndCommands<HeaderData,FrameData> & GetContext(const FGuid& InGuid)
	{
		FScopeLock Lock(&ContextMapCriticalSection);
		TSharedPtr<FContextAndCommands<HeaderData, FrameData >>* Context = ContextMap.Find(InGuid);
		return *(Context->Get());
	}
	
	void Close(const FGuid& InGuid)
	{
		FScopeLock Lock(&ContextMapCriticalSection);
		TSharedPtr<FContextAndCommands<HeaderData, FrameData >>* Context = ContextMap.Find(InGuid);
		if (Context)
		{
			(*Context)->Close();
			ContextMap.Remove(InGuid);
		}
	}

	void CloseContexts()
	{
		FScopeLock Lock(&ContextMapCriticalSection);

		for (auto& Entry : ContextMap)
		{
			TSharedPtr <FContextAndCommands<HeaderData, FrameData>> & Context = Entry.Value;
			Context->Close();
		}
		ContextMap.Empty();

	}
	/** The contexts we are using */
	TMap<FGuid, TSharedPtr<FContextAndCommands<HeaderData, FrameData>>> ContextMap;

	/** Thread safe bool for running the thread */
	FThreadSafeBool bRunning;

	/** Thread safe bool for stopping the thread */
	FThreadSafeBool bStopping;

	/** Read/write access flags */
	FThreadSafeBool bOpenForRead;
	FThreadSafeBool bOpenForWrite;

	/**  Lock */
	FCriticalSection ContextMapCriticalSection;

};

/** A class to asynchronously read and write to serialized frame debugger data files */
template <typename HeaderData,typename FrameData>
class  TMovieSceneSerializer
{
public:

	TMovieSceneSerializer();

	/** Get whether file exists */
	bool DoesFileExist(const FString& InFileName);

	/** Get the local capture directory */
	FString GetLocalCaptureDir();

	/** Set the local capture directory */
	void SetLocalCaptureDir(const FString& PathIn);
	
	/**
	* Open the serializer for writing
	* @return false if the operation was not successful with regards to opening the file, if we have saving turned off it will still return true.
	*/
	bool OpenForWrite(const FString& InFilename, HeaderData& Header, FText& OutFailReason);
	/** 
	 * Open the serializer for reading
	 * @return false if the operation was not successful
	 */

	bool OpenForRead(const FString& InFileName, HeaderData& OutHeader,  FText& OutFailReason);

	/** Close the serializer, flushes any commands and shuts down threads */
	void Close();

	/** Check whether this serializer is currently open */
	bool IsOpen() const;

	/** Write one frame data out to disk */
	void WriteFrameData(uint64 InFrameId, const FrameData &InFrame);

	/** Write array of frame data out to disk */
	void WriteFrameData(uint64 InFrameId, const TArray<FrameData> &InFrames);

	/** Read a frame range. Completion callback will be called on the game thread. */
	void ReadFramesAtFrameRange(uint64 InStartFrameId, uint64 InEndFrameId, TFunction<void()> InCompletionCallback);

	/** Query the range of data in the file. Completion callback will be called on the game thread. */
	void GetDataRanges(TFunction<void(uint64 /*InMinFrameId*/, uint64 /*InMaxFrameId*/)> InCompletionCallback);

	/** Get the number of sessions that were captured to this file */
	int32 GetNumSessions() { return SessionInfos.Num(); }

	/** Get the engine version the file was captured with. @see ISessionInstanceInfo */
	FGuid GetSessionId(int32 InSessionIndex) const { return SessionInfos[InSessionIndex].InstanceId; }

	/** Get the engine version the file was captured with. @see ISessionInstanceInfo */
	int32 GetBuildVersion(int32 InSessionIndex) const { return SessionInfos[InSessionIndex].BuildVersion; }

	/** Get the name of the device the file was captured on. @see ISessionInstanceInfo */
	FString GetDeviceName(int32 InSessionIndex) const { return SessionInfos[InSessionIndex].DeviceName; }

	/** Get the name of the platform the file was captured on. @see ISessionInstanceInfo */
	FString GetPlatformName(int32 InSessionIndex) const { return SessionInfos[InSessionIndex].PlatformName; }

	/** Get the name of the engine instance the file was captured on. @see ISessionInstanceInfo */
	FString GetInstanceName(int32 InSessionIndex) const { return SessionInfos[InSessionIndex].InstanceName; }

	/** Get the build date for the capture. @see ISessionInstanceInfo */
	FString GetBuildDate(int32 InSessionIndex) const { return SessionInfos[InSessionIndex].BuildDate; }

	/** How many frames have been written */
	int FramesWritten;

	/** Data from a read.  Be carefule if reading multiple times this data will get replaced and modified, so need to 
	come up with different way to get data in that case*/
	TArray<TMovieSceneSerializedFrame<FrameData>> ResultData;

	/** Thread we run on */
    static  SERIALIZEDRECORDERINTERFACE_API FRunnableThread* Thread;

	/** Runnable that the thread runs */
	static  SERIALIZEDRECORDERINTERFACE_API FMovieSceneSerializerRunnable<HeaderData, FrameData>* Runnable;
private:


	/** The session infos */
	TArray<FMovieSceneSessionInfo> SessionInfos;

	/** Local Directory*/
	FString LocalCaptureDir;

	/** Unique guid for this serializer */
	FGuid Guid;
};


template<typename HeaderData, typename FrameData>
TMovieSceneSerializer<HeaderData, FrameData>::TMovieSceneSerializer()
	: FramesWritten(0)
	, LocalCaptureDir(FPaths::ProjectSavedDir())
	, Guid(FGuid::NewGuid())
{
	
}
template<typename HeaderData, typename FrameData>
FString TMovieSceneSerializer<HeaderData, FrameData>::GetLocalCaptureDir()
{
	return LocalCaptureDir;
}
template<typename HeaderData, typename FrameData>
void  TMovieSceneSerializer<HeaderData, FrameData>::SetLocalCaptureDir(const FString& PathIn)
{
	LocalCaptureDir = PathIn;
}
template<typename HeaderData, typename FrameData>
bool TMovieSceneSerializer<HeaderData, FrameData>::DoesFileExist(const FString& AbsoluteFilePath)
{

	return IFileManager::Get().FileExists(*AbsoluteFilePath);
}

template<typename HeaderData, typename FrameData>
bool TMovieSceneSerializer<HeaderData, FrameData>::OpenForWrite(const FString& InFilename, HeaderData& Header, FText& OutFailReason)
{
	if (MovieSceneSerializationNamespace::bAutoSerialize)
	{
		// Get absolute file path
		FString SaveDirectory = GetLocalCaptureDir();
		FString AbsoluteFilePath = SaveDirectory + "/" + InFilename;
		FramesWritten = 0;
		// Create file handle and hand off to runnable thread
		IFileHandle* Handle = FPlatformFileManager::Get().GetPlatformFile().OpenWrite(*AbsoluteFilePath, false, true);
		if (Handle != nullptr)
		{
			UE_LOGF(MovieSceneSerialization, Display, "Opened Data cache: %ls", *AbsoluteFilePath);

			// Create runnable
			if (!Runnable)
			{
				Runnable = new FMovieSceneSerializerRunnable<HeaderData, FrameData>();
			}
			Runnable->AddContext(Guid, Handle, AbsoluteFilePath);
			Runnable->bOpenForRead = true;
			Runnable->bOpenForWrite = true;
			FMovieSceneSessionInfo SessionInfo(FApp::GetInstanceId(), FDateTime::UtcNow(), FNetworkVersion::GetNetworkCompatibleChangelist(), FPlatformProcess::ComputerName(),
				FPlatformProperties::PlatformName(), FApp::GetInstanceName(), FApp::GetBuildDate());
			SessionInfos.Emplace(SessionInfo);
			int64 ReadHeaderPos;

			if (Runnable->CheckHeader(Guid, false, SessionInfos, Header, ReadHeaderPos, OutFailReason))
			{
				// Create thread
				if (!Thread)
				{
					FString ThreadName = (Header.SerializedType.ToString());
					Thread = FRunnableThread::Create(Runnable, *ThreadName);
					UE_LOGF(MovieSceneSerialization, Display, "Created Thread For Type: %ls", *ThreadName);
				}

				return true;
			}
			else
			{
				Runnable->Close(Guid);
				/*
				Runnable->Stop();
				delete Runnable;
				Runnable = nullptr;
				*/
				return false;
			}
		}
		else
		{
			OutFailReason = NSLOCTEXT("MovieSceneSerializer", "FileOpenFailedToCreateArchive", "Cannot open file.\nFailed to create archive.");

			UE_LOGF(MovieSceneSerialization, Error, "Cannot open Movie Scene Serialization cache %ls. Failed to create archive.", *InFilename);
			return false;
		}
	}
	else
	{
		if (Runnable)
		{
			Runnable->bOpenForWrite = false;
		}
		return true;
	}
}

template<typename HeaderData, typename FrameData>
bool TMovieSceneSerializer<HeaderData, FrameData>::OpenForRead(const FString& AbsoluteFilePath, HeaderData& OutHeader, FText& OutFailReason)
{

	const bool bFileExists = IFileManager::Get().FileExists(*AbsoluteFilePath);

	// Create file handle and hand off to runnable thread
	IFileHandle* Handle = bFileExists ? FPlatformFileManager::Get().GetPlatformFile().OpenRead(*AbsoluteFilePath) : nullptr;
	if (Handle != nullptr)
	{
		UE_LOGF(MovieSceneSerialization, Display, "Opened Movie Scene Serialization cache: %ls", *AbsoluteFilePath);

		// Create runnable
		if (!Runnable)
		{
			Runnable = new FMovieSceneSerializerRunnable<HeaderData, FrameData>();
		}
		Runnable->AddContext(Guid, Handle, AbsoluteFilePath);

		Runnable->bOpenForRead = true;
		Runnable->bOpenForWrite = false;
		int64 ReadHeaderPos;

		if (Runnable->CheckHeader(Guid,bFileExists, SessionInfos, OutHeader, ReadHeaderPos,OutFailReason))
		{
			HeaderData Header = OutHeader;

			// Create thread
			if (!Thread)
			{
				FString ThreadName = (Header.SerializedType.ToString());
				Thread = FRunnableThread::Create(Runnable, *ThreadName);
				UE_LOGF(MovieSceneSerialization, Display, "Created Thread For Type: %ls", *ThreadName);
			}

			// Add a command to populate the dictionaries/maps from this existing file first
			FContextAndCommands<HeaderData,FrameData>& Context = Runnable->GetContext(Guid);
			Context.AddReadCommand([ReadHeaderPos](FMovieSceneSerializerContext& InContext)
			{
				InContext.FrameIdToFrameOffset.Reset();
				InContext.MinFrameId = 0xffffffffffffffff;
				InContext.MaxFrameId = 0;

				InContext.ArReader.Seek(ReadHeaderPos);

				while (true)
				{
					int64 FrameHeaderPos = InContext.ArReader.Tell();

					FMovieSceneSerializedFrameHeader Header;
					InContext.ArReader << Header;

					InContext.FrameIdToFrameOffset.Emplace(Header.FrameId, FrameHeaderPos);
					InContext.MinFrameId = FMath::Min(InContext.MinFrameId, Header.FrameId);
					InContext.MaxFrameId = FMath::Max(InContext.MaxFrameId, Header.FrameId);

					if (Header.NextFrameOffset != MovieSceneSerializationNamespace::InvalidOffset && Header.NextFrameOffset < InContext.ArReader.TotalSize())
					{
						InContext.ArReader.Seek(Header.NextFrameOffset);
					}
					else
					{
						break;
					}
				}

				InContext.ArReader.Seek(ReadHeaderPos);
			});

			return true;
		}
		else
		{
			/*
			Runnable->Stop();
			delete Runnable;
			Runnable = nullptr;
			*/
			return false;
		}
	}
	else
	{
		OutFailReason = NSLOCTEXT("MovieSceneSerializer", "FileOpenFailedToCreateArchive", "Cannot open file.\nFailed to create archive.");

		UE_LOGF(MovieSceneSerialization, Error, "Cannot open Movie Scene Serialization cache %ls. Failed to create archive.", *AbsoluteFilePath);
		return false;
	}
}
template<typename HeaderData, typename FrameData>
void TMovieSceneSerializer<HeaderData, FrameData>::Close()
{

	if (Thread && Runnable)
	{
		Runnable->Close(Guid);
		/* MZ TODO FIGURE WHEN AND HOW TO CLOSE AND DELETE 
		// Stop the thread (will flush)
		Runnable->Stop();
		Thread->WaitForCompletion();

		// Cleanup
		delete Thread;
		Thread = nullptr;

		delete Runnable;
		Runnable = nullptr;
		*/
	}
}
template<typename HeaderData, typename FrameData>
bool TMovieSceneSerializer<HeaderData, FrameData>::IsOpen() const
{
	return Thread && Runnable;
}

template<typename HeaderData, typename FrameData>
void TMovieSceneSerializer<HeaderData, FrameData>::GetDataRanges(TFunction<void(uint64 /*InMinFrameId*/, uint64 /*InMaxFrameId*/)> InCompletionCallback)
{
	if (Runnable != nullptr && Runnable->bOpenForRead)
	{
		FContextAndCommands<HeaderData,FrameData>& Context = Runnable->GetContext(Guid);
		Context.AddReadCommand([InCompletionCallback](FMovieSceneSerializerContext& InContext)
		{
			uint64 MinFrameId = 0, MaxFrameId = 0;
			MinFrameId = InContext.MinFrameId;
			MaxFrameId = InContext.MaxFrameId;

			AsyncTask(ENamedThreads::GameThread, [MinFrameId, MaxFrameId, InCompletionCallback]() mutable
			{
				InCompletionCallback(MinFrameId, MaxFrameId);
			});
		});
	}
}

template <typename HeaderData, typename FrameData>
void TMovieSceneSerializer<HeaderData, FrameData>::ReadFramesAtFrameRange(uint64 InStartFrameId, uint64 InEndFrameId, TFunction<void()> InCompletionCallback)
{
	check(InStartFrameId <= InEndFrameId);

	if (Runnable != nullptr && Runnable->bOpenForRead)
	{
		FContextAndCommands<HeaderData,FrameData>& Context = Runnable->GetContext(Guid);
		Context.AddReadCommand([this,InStartFrameId, InEndFrameId, InCompletionCallback](FMovieSceneSerializerContext& InContext)
		{
			// First we need to skip to our frame
			int32 FoundFirstFramePos = Algo::LowerBoundBy(InContext.FrameIdToFrameOffset, InStartFrameId, [](const TPair<uint64, int64>& InItem) { return InItem.Key; }, TLess<>());
			if (InContext.FrameIdToFrameOffset.IsValidIndex(FoundFirstFramePos))
			{
				int64 ReturnPos = InContext.ArReader.Tell();

				TArray<TMovieSceneSerializedFrame<FrameData>> TempFrames;

				InContext.ArReader.Seek(InContext.FrameIdToFrameOffset[FoundFirstFramePos].Value);
				while (true)
				{
					FMovieSceneSerializedFrameHeader Header;
					InContext.ArReader << Header;

					TMovieSceneSerializedFrame<FrameData>& TempFrame = TempFrames.Emplace_GetRef();
					InContext.ArReader << TempFrame;
					if (Header.NextFrameOffset != MovieSceneSerializationNamespace::InvalidOffset && Header.NextFrameOffset < InContext.ArReader.TotalSize() && Header.FrameId <= InEndFrameId)
					{
						InContext.ArReader.Seek(Header.NextFrameOffset);
					}
					else
					{
						break;
					}
				}

				InContext.ArReader.Seek(ReturnPos);

				AsyncTask(ENamedThreads::GameThread, [this, InCompletionCallback, TempFrames = MoveTemp(TempFrames)]() mutable
				{
					ResultData = MoveTemp(TempFrames);
					InCompletionCallback();
				});
			}
			else
			{
				AsyncTask(ENamedThreads::GameThread, [this,InCompletionCallback]()
				{
					TArray<TMovieSceneSerializedFrame<FrameData>> DummyFrames;
					ResultData = MoveTemp(DummyFrames);
					InCompletionCallback();
				});
			}
		});
	}
}

template<typename HeaderData, typename FrameData>
void TMovieSceneSerializer<HeaderData, FrameData>::WriteFrameData(uint64 InFrameId, const FrameData &InFrame)
{
	if (Runnable != nullptr && Runnable->bOpenForWrite)
	{
		FramesWritten += 1;
		FContextAndCommands<HeaderData,FrameData>& Context = Runnable->GetContext(Guid);
		Context.AddWriteCommand([InFrameId, InFrame](FMovieSceneSerializerContext& InContext)
		{
			FrameData CurrentBufferedWriteFrame = InFrame;
			// Seek to the end of the archive
			InContext.ArWriter.Seek(InContext.ArWriter.TotalSize());

			// Find our current offset
			int64 ThisFramePos = InContext.ArWriter.Tell();

			// Serialize out the next frame offset (currently invalid) and previous frame offset in a new header
			FMovieSceneSerializedFrameHeader Header(MovieSceneSerializationNamespace::InvalidOffset, InContext.LastFrameWritePos, InFrameId);
			InContext.ArWriter << Header;

			int64 FrameId = InFrameId;
			InContext.ArWriter << FrameId;
			// Serialize the buffered frame's body
			InContext.ArWriter << CurrentBufferedWriteFrame;
			// update caches
			InContext.FrameIdToFrameOffset.Emplace(InFrameId, ThisFramePos);
			InContext.MinFrameId = FMath::Min(InContext.MinFrameId, InFrameId);
			InContext.MaxFrameId = FMath::Max(InContext.MaxFrameId, InFrameId);

			// Serialize next frame pos to prev frame
			if (InContext.LastFrameWritePos != MovieSceneSerializationNamespace::InvalidOffset)
			{
				int64 ReturnPos = InContext.ArWriter.Tell();
				InContext.ArWriter.Seek(InContext.LastFrameWritePos + STRUCT_OFFSET(FMovieSceneSerializedFrameHeader, NextFrameOffset));
				InContext.ArWriter << ThisFramePos;
				InContext.ArWriter.Seek(ReturnPos);
			}

			InContext.LastFrameWritePos = ThisFramePos;
		});
	}
}


template<typename HeaderData, typename FrameData>
void TMovieSceneSerializer<HeaderData, FrameData>::WriteFrameData(uint64 InFrameId, const TArray<FrameData> &InFrames)
{
	if (Runnable != nullptr && Runnable->bOpenForWrite)
	{
		FramesWritten += InFrames.Num();
		FContextAndCommands<HeaderData,FrameData>& Context = Runnable->GetContext(Guid);
		Context.AddWriteCommand([InFrameId, InFrames](FMovieSceneSerializerContext& InContext)
		{
			uint64 FrameId = InFrameId;

			for (const FrameData& InFrame : InFrames)
			{
				FrameData CurrentBufferedWriteFrame = InFrame;
				// Seek to the end of the archive
				InContext.ArWriter.Seek(InContext.ArWriter.TotalSize());

				// Find our current offset
				int64 ThisFramePos = InContext.ArWriter.Tell();

				// Serialize out the next frame offset (currently invalid) and previous frame offset in a new header
				FMovieSceneSerializedFrameHeader Header(MovieSceneSerializationNamespace::InvalidOffset, InContext.LastFrameWritePos, FrameId);
				InContext.ArWriter << Header;

				InContext.ArWriter << FrameId;
				// Serialize the buffered frame's body
				InContext.ArWriter << CurrentBufferedWriteFrame;
				// update caches
				InContext.FrameIdToFrameOffset.Emplace(FrameId, ThisFramePos);
				InContext.MinFrameId = FMath::Min(InContext.MinFrameId, FrameId);
				InContext.MaxFrameId = FMath::Max(InContext.MaxFrameId, FrameId);

				// Serialize next frame pos to prev frame
				if (InContext.LastFrameWritePos != MovieSceneSerializationNamespace::InvalidOffset)
				{
					int64 ReturnPos = InContext.ArWriter.Tell();
					InContext.ArWriter.Seek(InContext.LastFrameWritePos + STRUCT_OFFSET(FMovieSceneSerializedFrameHeader, NextFrameOffset));
					InContext.ArWriter << ThisFramePos;
					InContext.ArWriter.Seek(ReturnPos);
				}
				++FrameId;
				InContext.LastFrameWritePos = ThisFramePos;
			}
		});
	}
}
