// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "UObject/UnrealType.h"
#include "MovieSceneSectionSerialization.h"

//Though we are embedding the serialized type of the data into the file name they may not match so we 
//go ahead and read the type from the file by mocking a header that just reads in the type.
struct FSerializedTypeFileHeader
{
	static const int32 cVersion = 1;

	FSerializedTypeFileHeader() : Version(cVersion)
	{
	}


	friend FArchive& operator<<(FArchive& Ar, FSerializedTypeFileHeader& Header)
	{
		Ar << Header.Version;
		Ar << Header.SerializedType;

		return Ar;
	}

	int32 Version;
	FName SerializedType;
};
