// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "UObject/UnrealType.h"
#include "MovieSceneSectionSerialization.h"


struct FSpawnFileHeader
{
	static const int32 cVersion = 1;

	FSpawnFileHeader() : Version(cVersion)
	{
	}


	FSpawnFileHeader(FFrameRate &InTickResolution, const FName& InSerializedType, const FGuid& InGuid)
		: Version(cVersion)
		, SerializedType(InSerializedType)
		, Guid(InGuid)
		, TickResolution(InTickResolution)

	{};


	friend FArchive& operator<<(FArchive& Ar, FSpawnFileHeader& Header)
	{
		Ar << Header.Version;
		Ar << Header.SerializedType;
		Ar << Header.Guid;
		Ar << Header.TickResolution.Numerator;
		Ar << Header.TickResolution.Denominator;
		return Ar;
	}

	//DATA
	int32 Version;
	FName SerializedType;
	FGuid Guid;
	FFrameRate TickResolution;
};


//since we can't use function templates with TFunctions need to use a variant or union.
struct FSpawnProperty
{
	FSpawnProperty() = default;
	FSpawnProperty( FFrameNumber InKeyTime, bool InVal)
	{
		Time = InKeyTime;
		bVal = InVal;
	}
	friend FArchive& operator<<(FArchive& Ar, FSpawnProperty& Property)
	{
		Ar << Property.Time;
		Ar << Property.bVal;
		return Ar;
	}

	//DATA
	FFrameNumber Time;
	bool bVal;
};


using FSpawnSerializedFrame = TMovieSceneSerializedFrame<FSpawnProperty>;
using FSpawnSerializer = TMovieSceneSerializer<FSpawnFileHeader, FSpawnProperty>;