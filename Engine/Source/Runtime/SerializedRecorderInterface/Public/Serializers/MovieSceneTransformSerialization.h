// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameRate.h"
#include "UObject/UnrealType.h"
#include "MovieSceneSectionSerialization.h"


struct FTransformFileHeader
{
	static const int32 cVersion = 1;

	FTransformFileHeader() : Version(cVersion)
	{
	}

	FTransformFileHeader(FFrameRate &InTickResolution, const FName& InSerializedType, const FGuid& InGuid)
		: Version(cVersion)
		, SerializedType(InSerializedType)
		, Guid(InGuid)
		, TickResolution(InTickResolution)

	{};


	friend FArchive& operator<<(FArchive& Ar, FTransformFileHeader& Header)
	{
		Ar << Header.Version;
		Ar << Header.SerializedType;
		Ar << Header.Guid;
		Ar << Header.TickResolution.Numerator;
		Ar << Header.TickResolution.Denominator;

		return Ar;
	}

	//DATA
	int32 Version;
	FName SerializedType;
	FGuid Guid;
	FFrameRate TickResolution;

};

struct FSerializedTransform
{
	FSerializedTransform() = default;

	FSerializedTransform(const FTransform& InTransform, FFrameNumber InKeyTime)
	{
		Set(InTransform, InKeyTime);
	}
	void Set(const FTransform& InTransform, FFrameNumber InKeyTime)
	{
		Time = InKeyTime;
		Values[0] = InTransform.GetTranslation().X;
		Values[1] = InTransform.GetTranslation().Y;
		Values[2] = InTransform.GetTranslation().Z;

		FRotator WoundRoation = InTransform.Rotator();
		Values[3] = WoundRoation.Roll;
		Values[4] = WoundRoation.Pitch;
		Values[5] = WoundRoation.Yaw;

		Values[6] = InTransform.GetScale3D().X;
		Values[7] = InTransform.GetScale3D().Y;
		Values[8] = InTransform.GetScale3D().Z;
	}

	friend FArchive& operator<<(FArchive& Ar, FSerializedTransform& Transform)
	{
		Ar << Transform.Time;
		Ar << Transform.Values[0];
		Ar << Transform.Values[1];
		Ar << Transform.Values[2];
		Ar << Transform.Values[3];
		Ar << Transform.Values[4];
		Ar << Transform.Values[5];
		Ar << Transform.Values[6];
		Ar << Transform.Values[7];
		Ar << Transform.Values[8];
		return Ar;
	}

	FFrameNumber Time;
	double Values[9];   //location, rotation, scale.
};

using FTransformSerializedFrame = TMovieSceneSerializedFrame<FSerializedTransform>;
using FTransformSerializer = TMovieSceneSerializer<FTransformFileHeader, FSerializedTransform>;
