// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SerializedRecorderInterface : ModuleRules
{
    public SerializedRecorderInterface(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject",
                "LiveLinkInterface",

            }
        );
	}
}
